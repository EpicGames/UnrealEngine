// Copyright Epic Games, Inc. All Rights Reserved.

#include "HDRIBackdrop.h"
#include "HDRIBackdropPlacement.h"
#include "HDRIBackdropStyle.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FHDRIBackdropModule"

void FHDRIBackdropModule::StartupModule()
{
	FHDRIBackdropStyle::Initialize();
	FHDRIBackdropPlacement::RegisterPlacement();
}

void FHDRIBackdropModule::ShutdownModule()
{
	FHDRIBackdropStyle::Shutdown();
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FHDRIBackdropModule, HDRIBackdrop)
