// Copyright Epic Games, Inc. All Rights Reserved.

#include "HDRIBackdropPlacement.h"
#include "Engine/Blueprint.h"
#include "IPlacementModeModule.h"
#include "ActorFactories/ActorFactoryBlueprint.h"

#define LOCTEXT_NAMESPACE "FHDRIBackdropModule"

void FHDRIBackdropPlacement::RegisterPlacement()
{	
	UBlueprint* HDRIBackdrop = Cast<UBlueprint>(FSoftObjectPath(TEXT("/HDRIBackdrop/Blueprints/HDRIBackdrop.HDRIBackdrop")).TryLoad());
	if (HDRIBackdrop == nullptr)
	{
		return;
	}
	
	IPlacementModeModule& PlacementModeModule = IPlacementModeModule::Get();
	FPlacementCategoryInfo Info = *PlacementModeModule.GetRegisteredPlacementCategory(FBuiltInPlacementCategories::Lights());

	FPlaceableItem* BPPlacement = new FPlaceableItem(
		*UActorFactoryBlueprint::StaticClass(),
		FAssetData(HDRIBackdrop, true),
		FName("HDRIBackdrop.ModesThumbnail"),
		FName("HDRIBackdrop.ModesIcon"),
		TOptional<FLinearColor>(),
		TOptional<int32>(),
		NSLOCTEXT("PlacementMode", "HDRI Backdrop", "HDRI Backdrop")
	);

	IPlacementModeModule::Get().RegisterPlaceableItem( Info.UniqueHandle, MakeShareable(BPPlacement) );
}

#undef LOCTEXT_NAMESPACE
