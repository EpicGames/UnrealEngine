// Copyright Epic Games, Inc. All Rights Reserved.

#include "HDRIBackdropStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyleRegistry.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"

TSharedPtr<FSlateStyleSet> FHDRIBackdropStyle::StyleSet;

void FHDRIBackdropStyle::Initialize()
{
	if (StyleSet.IsValid())
	{
		return;
	}

	StyleSet = MakeShared<FSlateStyleSet>(FName("HDRIBackdropStyle"));

	StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	FString IconPath = IPluginManager::Get().FindPlugin(TEXT("HDRIBackdrop"))->GetBaseDir() + TEXT("/Resources/HDRIBackdrop.png");
	StyleSet->Set("HDRIBackdrop.ModesThumbnail", new FSlateImageBrush(IconPath, FVector2D(40.0f, 40.0f)));
	StyleSet->Set("HDRIBackdrop.ModesIcon", new FSlateImageBrush(IconPath, FVector2D(16.0f, 16.0f)));

	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
}

void FHDRIBackdropStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}
