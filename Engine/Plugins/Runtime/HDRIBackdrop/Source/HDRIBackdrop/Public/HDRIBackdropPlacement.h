// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FHDRIBackdropPlacement
{
public:
	static void RegisterPlacement();
};